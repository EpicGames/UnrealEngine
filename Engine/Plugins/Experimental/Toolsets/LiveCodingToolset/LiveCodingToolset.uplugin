{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "LiveCodingToolset",
	"Description": "Live Coding compile toolset.",
	"NoRedist": true,
	"EditorOnly": true,
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "LiveCodingToolset",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": ["Editor"]
		},
		{
			"Name": "LiveCodingToolsetTests",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"TargetAllowList": ["Editor"]
		}
	],
	"Plugins": [
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		}
	]
}
