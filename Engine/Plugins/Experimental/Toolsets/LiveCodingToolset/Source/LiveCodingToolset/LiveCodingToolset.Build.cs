// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class LiveCodingToolset : ModuleRules
	{
		public LiveCodingToolset(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"ToolsetRegistry",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"EditorSubsystem",
					"Engine",
					"UnrealEd",
				}
			);

			if (Target.bWithLiveCoding)
			{
				PrivateDependencyModuleNames.Add("LiveCoding");
			}
		}
	}
}
