// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveCodingToolset.h"

#include "Logging/LogMacros.h"

#if WITH_LIVE_CODING
#include "HAL/FileManager.h"
#include "ILiveCodingModule.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogLiveCodingToolset, Log, All);

#if WITH_LIVE_CODING
namespace
{
	/** Temporary output device that captures LogLiveCoding messages during compilation.
	 *  Live Coding runs its compile pipeline off the game thread and logs from worker
	 *  threads, so Serialize must be safe to call concurrently. Output mutations are
	 *  guarded by a critical section. */
	class FLiveCodingOutputCollector : public FOutputDevice
	{
	public:
		virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override
		{
			static const FName LiveCodingCategory(TEXT("LogLiveCoding"));
			if (Category != LiveCodingCategory)
			{
				return;
			}
			FScopeLock Lock(&OutputCriticalSection);
			if (!Output.IsEmpty())
			{
				Output += TEXT("\n");
			}
			Output += V;
		}

		virtual bool CanBeUsedOnAnyThread() const override { return true; }

		FString ConsumeOutput()
		{
			FScopeLock Lock(&OutputCriticalSection);
			return MoveTemp(Output);
		}

	private:
		FCriticalSection OutputCriticalSection;
		FString Output;
	};

	FString CompileResultToString(ELiveCodingCompileResult Result)
	{
		switch (Result)
		{
		case ELiveCodingCompileResult::Success:            return TEXT("Success");
		case ELiveCodingCompileResult::NoChanges:          return TEXT("NoChanges");
		case ELiveCodingCompileResult::Failure:            return TEXT("Failure");
		case ELiveCodingCompileResult::CompileStillActive: return TEXT("CompileStillActive");
		case ELiveCodingCompileResult::NotStarted:         return TEXT("NotStarted");
		case ELiveCodingCompileResult::Cancelled:          return TEXT("Cancelled");
		case ELiveCodingCompileResult::InProgress:         return TEXT("InProgress");
		default:                                           return TEXT("Unknown");
		}
	}

	FString GetUBTLogPath()
	{
		return FPaths::Combine(FPaths::EngineDir(), TEXT("Programs"), TEXT("UnrealBuildTool"), TEXT("Log.txt"));
	}

	/** Extract lines containing MSVC diagnostics from UBT log text. */
	FString ExtractDiagnostics(const FString& LogText)
	{
		FString Diagnostics;
		TArray<FString> Lines;
		LogText.ParseIntoArrayLines(Lines);

		for (const FString& Line : Lines)
		{
			if (Line.Contains(TEXT("): error ")) || Line.Contains(TEXT("): warning ")) || Line.Contains(TEXT("): fatal error ")))
			{
				if (!Diagnostics.IsEmpty())
				{
					Diagnostics += TEXT("\n");
				}
				Diagnostics += Line;
			}
		}

		return Diagnostics;
	}

}
#endif // WITH_LIVE_CODING

FString ULiveCodingToolset::CompileLiveCoding()
{
#if WITH_LIVE_CODING
	UE_LOG(LogLiveCodingToolset, Log, TEXT("CompileLiveCoding: entry"));

	ILiveCodingModule* LiveCoding = FModuleManager::GetModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME);
	if (!LiveCoding)
	{
		UE_LOG(LogLiveCodingToolset, Warning, TEXT("CompileLiveCoding: Live Coding module is not loaded"));
		return TEXT("Error: Live Coding module is not loaded. Ensure Live Coding is enabled in Editor Preferences.");
	}
	if (!LiveCoding->IsEnabledForSession())
	{
		UE_LOG(LogLiveCodingToolset, Warning, TEXT("CompileLiveCoding: Live Coding is not enabled for this session"));
		return TEXT("Error: Live Coding is not enabled for this session. Enable it in Editor Preferences.");
	}

	// Snapshot the UBT Log.txt modification time before compile. UBT backs up and
	// rewrites Log.txt on each invocation (see EpicGames.Core Log.BackupLogFile),
	// so a changed timestamp is the signal that this compile actually produced a
	// fresh log. If the timestamp is unchanged, the prior log is stale and must
	// not leak into the result.
	const FString UBTLogPath = GetUBTLogPath();
	const FDateTime UBTLogTimestampBeforeCompile = IFileManager::Get().GetTimeStamp(*UBTLogPath);

	// Capture LogLiveCoding output during compilation (Live Coding hooks
	// surface compiler/linker output via this category).
	FLiveCodingOutputCollector OutputCollector;
	GLog->AddOutputDevice(&OutputCollector);

	ELiveCodingCompileResult Result = ELiveCodingCompileResult::NotStarted;
	const bool bStarted = LiveCoding->Compile(ELiveCodingCompileFlags::WaitForCompletion, &Result);

	GLog->RemoveOutputDevice(&OutputCollector);

	FString ResultString;
	if (!bStarted)
	{
		ResultString = TEXT("Result: CompileNotStarted");
	}
	else
	{
		ResultString = FString::Printf(TEXT("Result: %s"), *CompileResultToString(Result));
	}

	UE_LOG(LogLiveCodingToolset, Log, TEXT("CompileLiveCoding: %s"), *ResultString);

	const FString CapturedOutput = OutputCollector.ConsumeOutput();
	if (!CapturedOutput.IsEmpty())
	{
		ResultString += TEXT("\n\nCompile Output:\n");
		ResultString += CapturedOutput;
	}

	// Supplement hook output with any MSVC diagnostics that UBT wrote to its own
	// log during this compile. Only read the log when its timestamp has changed,
	// so a prior runs diagnostics cannot leak into the result.
	const FDateTime UBTLogTimestampAfterCompile = IFileManager::Get().GetTimeStamp(*UBTLogPath);
	if (UBTLogTimestampAfterCompile != FDateTime::MinValue() && UBTLogTimestampAfterCompile != UBTLogTimestampBeforeCompile)
	{
		FString LogContent;
		if (FFileHelper::LoadFileToString(LogContent, *UBTLogPath))
		{
			const FString Diagnostics = ExtractDiagnostics(LogContent);
			if (!Diagnostics.IsEmpty())
			{
				ResultString += TEXT("\n\nCompiler Diagnostics:\n");
				ResultString += Diagnostics;
			}
		}
	}

	return ResultString;
#else
	UE_LOG(LogLiveCodingToolset, Warning, TEXT("CompileLiveCoding: Live Coding is not available in this build configuration"));
	return TEXT("Error: Live Coding is not available in this build configuration.");
#endif // WITH_LIVE_CODING
}
