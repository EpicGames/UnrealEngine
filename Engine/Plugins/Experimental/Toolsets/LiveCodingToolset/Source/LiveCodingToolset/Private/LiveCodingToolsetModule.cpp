// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveCodingToolsetModule.h"

#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FLiveCodingToolsetModule, LiveCodingToolset)

void FLiveCodingToolsetModule::StartupModule()
{
}

void FLiveCodingToolsetModule::ShutdownModule()
{
}
