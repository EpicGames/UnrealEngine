// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveCodingToolsetSubsystem.h"

#include "Editor.h"
#include "LiveCodingToolset.h"
#include "Subsystems/SubsystemCollection.h"
#include "ToolsetRegistry/ToolsetRegistrySubsystem.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

static bool bEnableLiveCodingToolset = true;

static void OnEnableLiveCodingToolsetChanged(IConsoleVariable* Variable)
{
	if (GEditor == nullptr)
	{
		return;
	}

	if (ULiveCodingToolsetSubsystem* Subsystem = GEditor->GetEditorSubsystem<ULiveCodingToolsetSubsystem>())
	{
		Subsystem->SetToolsetEnabled(Variable->GetBool());
	}
}

static FAutoConsoleVariableRef CVarEnableLiveCodingToolset(
	TEXT("LiveCodingToolset.Enable"),
	bEnableLiveCodingToolset,
	TEXT("Enable or disable LiveCodingToolset registration. When disabled, its tools will not appear in the MCP tool list."),
	FConsoleVariableDelegate::CreateStatic(&OnEnableLiveCodingToolsetChanged),
	ECVF_Default);

void ULiveCodingToolsetSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	// Ensure UToolsetRegistrySubsystem is initialized before we call RegisterToolsetClass.
	// RegisterToolsetClass silently no-ops if the registry subsystem is not yet ready.
	Collection.InitializeDependency<UToolsetRegistrySubsystem>();
	SetToolsetEnabled(bEnableLiveCodingToolset);
}

void ULiveCodingToolsetSubsystem::Deinitialize()
{
	SetToolsetEnabled(false);
	Super::Deinitialize();
}

void ULiveCodingToolsetSubsystem::SetToolsetEnabled(bool bEnabled)
{
	if (bEnabled && !bToolsetRegistered)
	{
		UToolsetRegistry::RegisterToolsetClass(ULiveCodingToolset::StaticClass());
		bToolsetRegistered = true;
	}
	else if (!bEnabled && bToolsetRegistered)
	{
		UToolsetRegistry::UnregisterToolsetClass(ULiveCodingToolset::StaticClass());
		bToolsetRegistered = false;
	}
}
