// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolsetRegistry/ToolsetDefinition.h"

#include "LiveCodingToolset.generated.h"

/**
 * Live Coding compile toolset.
 *
 * Exposes an MCP tool that triggers a Live Coding compile from the running
 * editor and waits for the result, returning the compile status together with
 * any captured LogLiveCoding output and UBT compiler diagnostics.
 */
UCLASS(BlueprintType, MinimalAPI)
class ULiveCodingToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:

	/** Triggers a Live Coding compile and waits for the result.
	 * Live Coding must be enabled in Editor Preferences.
	 * @return The compile result status followed by any compiler output (errors, warnings, etc). */
	UFUNCTION(meta = (AICallable), Category = "LiveCodingToolset")
	static LIVECODINGTOOLSET_API FString CompileLiveCoding();
};
