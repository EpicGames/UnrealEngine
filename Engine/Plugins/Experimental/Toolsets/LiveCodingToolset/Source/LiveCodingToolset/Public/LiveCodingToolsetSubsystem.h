// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"

#include "LiveCodingToolsetSubsystem.generated.h"

class FSubsystemCollectionBase;

/**
 * Registers the LiveCodingToolset with UToolsetRegistry for the lifetime of the editor.
 * Registration can be toggled at runtime via the LiveCodingToolset.Enable CVar.
 */
UCLASS(MinimalAPI)
class ULiveCodingToolsetSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:

	virtual void Initialize(FSubsystemCollectionBase& Collection) override;

	virtual void Deinitialize() override;

	/** Register or unregister the toolset with UToolsetRegistry at runtime. */
	void SetToolsetEnabled(bool bEnabled);

private:

	bool bToolsetRegistered = false;
};
