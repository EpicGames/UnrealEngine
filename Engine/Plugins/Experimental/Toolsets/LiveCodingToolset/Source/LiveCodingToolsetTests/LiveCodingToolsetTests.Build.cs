// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class LiveCodingToolsetTests : ModuleRules
	{
		public LiveCodingToolsetTests(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"Core",
				"CoreUObject",
				"CQTest",
				"Engine",
				"LiveCodingToolset",
				"ToolsetRegistry",
				"UnrealEd",
			});
		}
	}
}
