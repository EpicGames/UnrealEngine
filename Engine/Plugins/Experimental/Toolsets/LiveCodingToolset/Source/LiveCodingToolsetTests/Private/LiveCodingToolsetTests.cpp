// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTest.h"

#include "Editor.h"
#include "LiveCodingToolset.h"
#include "LiveCodingToolsetSubsystem.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

TEST_CLASS(LiveCodingToolsetTest, "AI.Toolsets.LiveCodingToolset")
{
	TEST_METHOD(Subsystem_IsAvailable)
	{
		ULiveCodingToolsetSubsystem* Subsystem = GEditor->GetEditorSubsystem<ULiveCodingToolsetSubsystem>();
		ASSERT_THAT(IsNotNull(Subsystem));
	}

	TEST_METHOD(Subsystem_RegistersToolsetByDefault)
	{
		// The subsystem registers ULiveCodingToolset in Initialize when the
		// LiveCodingToolset.Enable CVar is true (the default).
		ASSERT_THAT(IsTrue(UToolsetRegistry::IsToolsetClassRegistered(ULiveCodingToolset::StaticClass())));
	}

	TEST_METHOD(CompileLiveCoding_ReturnsNonEmptyString)
	{
		// Regardless of whether Live Coding is enabled for the session, the
		// tool must always return a descriptive string (success, error, or
		// "not available" stub) rather than an empty one.
		const FString Result = ULiveCodingToolset::CompileLiveCoding();
		ASSERT_THAT(IsFalse(Result.IsEmpty()));
	}
};
