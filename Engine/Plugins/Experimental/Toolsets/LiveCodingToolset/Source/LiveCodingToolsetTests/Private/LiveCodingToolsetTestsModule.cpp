// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

class FLiveCodingToolsetTestsModule : public IModuleInterface
{
};

IMPLEMENT_MODULE(FLiveCodingToolsetTestsModule, LiveCodingToolsetTests)
